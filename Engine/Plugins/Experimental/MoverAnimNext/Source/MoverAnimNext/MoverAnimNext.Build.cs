// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MoverAnimNext : ModuleRules
{
	public MoverAnimNext(ReadOnlyTargetRules Target) : base(Target)
	{

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Mover",
				"UAF",
				"PoseSearch"
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"RigVM",
				"Engine",
				"PhysicsCore",
				"DeveloperSettings"
			}
			);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"RigVMDeveloper"
				}
			);
		}

		SetupGameplayDebuggerSupport(Target);
	}
}
