// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_GenerateMoverTrajectory.h"

#include "Component/AnimNextComponent.h"
#include "MoverComponent.h"
#include "MoverPoseSearchTrajectoryPredictor.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"
#include "UAFLogging.h"

FRigUnit_GenerateMoverTrajectory_Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigUnit_GenerateMoverTrajectory_Execute);
	using namespace UE::UAF;

	// Note this logic is based on UPoseSearchTrajectoryLibrary::PoseSearchGeneratePredictorTrajectory

	FPoseSearchTrajectoryData::FSampling TrajectoryDataSampling;
	TrajectoryDataSampling.NumHistorySamples = NumHistorySamples;
	TrajectoryDataSampling.SecondsPerHistorySample = HistorySamplingInterval;
	TrajectoryDataSampling.NumPredictionSamples = NumPredictionSamples;
	TrajectoryDataSampling.SecondsPerPredictionSample = PredictionSamplingInterval;

	// TODO: handle controller yaw
	//TrajectoryDataState.DesiredControllerYawLastUpdate = InOutDesiredControllerYawLastUpdate;

	FVector CurrentPosition = FVector::ZeroVector;
	FVector CurrentVelocity = FVector::ZeroVector;
	FQuat CurrentFacing = FQuat::Identity;

	if (MoverComponent == nullptr)
	{
		// Cannot execute, abort
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not generate trajectory - Mover component is not valid."));
		return;
	}

	UMoverTrajectoryPredictor::GetCurrentState(*MoverComponent, CurrentPosition, CurrentFacing, CurrentVelocity);
	UPoseSearchTrajectoryLibrary::InitTrajectorySamples(InOutTrajectory, CurrentPosition, CurrentFacing, TrajectoryDataSampling, DeltaTime);
	UPoseSearchTrajectoryLibrary::UpdateHistory_TransformHistory(InOutTrajectory, CurrentPosition, CurrentVelocity, TrajectoryDataSampling, DeltaTime);
	
	// Set the current position at i == NumHistorySamples at t == delta time
	// Remember: t == 0 is the previous position, t == delta time is the current frame's position
	// assuming we call this method after the movement component has updated to a new position
	InOutTrajectory.Samples[NumHistorySamples].TimeInSeconds = DeltaTime;
	InOutTrajectory.Samples[NumHistorySamples].Position = CurrentPosition;
	InOutTrajectory.Samples[NumHistorySamples].Facing = CurrentFacing;
	
	UMoverTrajectoryPredictor::Predict(*MoverComponent, InOutTrajectory,
		NumPredictionSamples, PredictionSamplingInterval, NumHistorySamples, static_cast<float>(MoverSamplingFrameRate.AsInterval()));
}
