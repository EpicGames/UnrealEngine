// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Animation/TrajectoryTypes.h"
#include "Misc/FrameRate.h"

#include "RigUnit_GenerateMoverTrajectory.generated.h"

class UMoverComponent;

/*
 * A utility node to generate a trajectory from a mover component
 */
USTRUCT(meta = (DisplayName = "Generate Trajectory from Mover", Category = "Mover", NodeColor = "0, 1, 1", Keywords = "Mover, Trajectory"))
struct FRigUnit_GenerateMoverTrajectory : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The mover component to generate the trajectory for
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	TObjectPtr<UMoverComponent> MoverComponent;

	// This should be the most recent simulation time that was used to get us to our current state
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float DeltaTime = 0.f;

	// The sampling interval to use for the history
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float HistorySamplingInterval = -1.f;

	// The upper limit of samples to use for the history
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	int32 NumHistorySamples = 30;

	// The interval to use for prediction (in seconds)
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float PredictionSamplingInterval = 0.1f;

	// The upper limit of samples to use for prediction
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	int32 NumPredictionSamples = 15;

	// Sampling frame rate to query the mover, doesn't necessarily match the PredictionSamplingInterval which is used to write to the trajectory
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	FFrameRate MoverSamplingFrameRate = FFrameRate(60, 1);

	// The resulting trajectory 
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input, Output))
	FTransformTrajectory InOutTrajectory;
	
	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};
