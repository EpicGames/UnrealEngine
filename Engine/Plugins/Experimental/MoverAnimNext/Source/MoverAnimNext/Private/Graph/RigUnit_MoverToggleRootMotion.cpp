// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_MoverToggleRootMotion.h"

#include "DefaultMovementSet/LayeredMoves/RootMotionAttributeLayeredMove.h"
#include "MoverComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "UAFLogging.h"

FRigUnit_MoverToggleRootMotion_Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigUnit_MoverToggleRootMotion_Execute);
	using namespace UE::UAF;

	if (!MoverComponent || !MoverComponent->HasBeenInitialized())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not toggle root motion - No valid mover component provided, or component has not been initialized yet."));
		return;
	}

	TWeakObjectPtr<UMoverComponent> WeakMoverComponent = MoverComponent;
	auto MoverComponentTask = [bRootMotionEnabled, WeakMoverComponent]()
	{
		if (UMoverComponent* MoverComponent = WeakMoverComponent.Get())
		{
			if (bRootMotionEnabled)
			{
				MoverComponent->QueueLayeredMove(MakeShared<FLayeredMove_RootMotionAttribute>());
			}
			else
			{
				MoverComponent->CancelFeaturesWithTag(Mover_AnimRootMotion_MeshAttribute);
			}
		}
	};

	if (bToggleOnGameThread)
	{
		FAnimNextModuleInstance::RunTaskOnGameThread(MoverComponentTask);
	}
	else
	{
		MoverComponentTask();
	}
}
