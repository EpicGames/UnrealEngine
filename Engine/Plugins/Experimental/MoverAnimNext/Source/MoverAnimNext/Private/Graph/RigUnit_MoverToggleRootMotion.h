// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"

#include "RigUnit_MoverToggleRootMotion.generated.h"

class UMoverComponent;

/**
 * RigVM Node that can be used to toggle mover root motion. The toggle will occur next frame.
 * You should also update your animation tick dependencies to so that animation runs before mover when this is toggled.
 * 
 * For example. Within UAF during PrePhysics: 
 * If (bRootMotionEnabled) -> AddDependency : MoverComponent, Before, SimulateMovment -> RemoveDependency : MoverComponent, After, SimulateMovment
 * If (!bRootMotionEnabled) -> RemoveDependency : MoverComponent, Before, SimulateMovment -> AddDependency : MoverComponent, After, SimulateMovment
 * 
 * Where: If / AddDependency / RemoveDependency are nodes in your RigVM UAF PrePhysics graph.
 */
USTRUCT(meta = (DisplayName = "Mover Toggle Root Motion", Category = "Mover", NodeColor = "0, 1, 1", Keywords = "Mover, Root Motion, Enable, Disable"))
struct FRigUnit_MoverToggleRootMotion : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	/** Mover to toggle root motion on */
	UPROPERTY(EditAnywhere, Category = "Mover", meta = (Input))
	TObjectPtr<UMoverComponent> MoverComponent = nullptr;

	/** True if we want to enable root motion on the mover */
	UPROPERTY(EditAnywhere, Category = "Mover", meta = (Input))
	bool bRootMotionEnabled = false;

	/** True if we want to delay the toggle to next frame on game thread. This allows for tick dependencies to change for next frame */
	UPROPERTY(EditAnywhere, Category = "Mover", AdvancedDisplay, meta = (Input))
	bool bToggleOnGameThread = true;

	// Execution input / output
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "Mover", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};
