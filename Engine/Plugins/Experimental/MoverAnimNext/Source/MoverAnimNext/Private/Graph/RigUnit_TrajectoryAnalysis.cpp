// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_TrajectoryAnalysis.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"

FRigUnit_GetTrajectorySampleAtTime_Execute()
{
	UPoseSearchTrajectoryLibrary::GetTransformTrajectorySampleAtTime(InTrajectory, Time, OutTrajectorySample, bExtrapolate);
}

FRigUnit_GetTrajectoryVelocity_Execute()
{
	UPoseSearchTrajectoryLibrary::GetTransformTrajectoryVelocity(InTrajectory, Time1, Time2, OutVelocity, bExtrapolate);
}

FRigUnit_GetTrajectoryAngularVelocity_Execute()
{
	UPoseSearchTrajectoryLibrary::GetTransformTrajectoryAngularVelocity(InTrajectory, Time1, Time2, OutAngularVelocity, bExtrapolate);
}