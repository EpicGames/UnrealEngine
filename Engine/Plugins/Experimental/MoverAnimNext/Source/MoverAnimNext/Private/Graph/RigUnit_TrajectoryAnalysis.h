// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Animation/TrajectoryTypes.h"

#include "RigUnit_TrajectoryAnalysis.generated.h"

/*
 * Returns a sample of a trajectory at a given time
 */
USTRUCT(meta = (DisplayName = "Get Trajectory Sample At Time", Category = "Trajectory", NodeColor = "0, 1, 1", Keywords = "Trajectory"))
struct FRigUnit_GetTrajectorySampleAtTime : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The resulting trajectory sample
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Output))
	FTransformTrajectorySample OutTrajectorySample;

	// The input trajectory to analyze
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	FTransformTrajectory InTrajectory;

	// The time at which to analyze the trajectory
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float Time = 0.0f;
	
	// If True the trajectory will be extrapolated beyond its bounds
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	bool bExtrapolate = false;
};

/*
 * Computes the velocity between two times on a trajectory
 */
USTRUCT(meta = (DisplayName = "Get Trajectory Velocity", Category = "Trajectory", NodeColor = "0, 1, 1", Keywords = "Trajectory"))
struct FRigUnit_GetTrajectoryVelocity : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The input trajectory to analyze
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	FTransformTrajectory InTrajectory;

	// The resulting velocity between the two provided times
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Output))
	FVector OutVelocity = FVector::ZeroVector;

	// The first time to analyze
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float Time1 = 0.0f;

	// The second time to analyze
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float Time2 = 0.0f;

	// If True the trajectory will be extrapolated beyond its bounds
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	bool bExtrapolate = false;
};

/*
 * Computes the angular velocity between two times on a trajectory
 */
USTRUCT(meta = (DisplayName = "Get Trajectory Angular Velocity", Category = "Trajectory", NodeColor = "0, 1, 1", Keywords = "Trajectory"))
struct FRigUnit_GetTrajectoryAngularVelocity : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The input trajectory to analyze
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	FTransformTrajectory InTrajectory;

	// The resulting angular velocity between the two provided times
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Output))
	FVector OutAngularVelocity = FVector::ZeroVector;

	// The first time to analyze
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float Time1 = 0.0f;

	// The second time to analyze
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float Time2 = 0.0f;

	// If True the trajectory will be extrapolated beyond its bounds
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	bool bExtrapolate = false;
};