// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTrait_ModuleEventDependency_MoverComponentTickFunctions.h"
#include "Backends/MoverStandaloneLiaison.h"


#if WITH_EDITOR
FString FRigVMTrait_ModuleEventDependency_MoverComponentTickFunctions::GetDisplayName() const
{
	return StaticStruct()->GetDisplayNameText().ToString();
}
#endif // WITH_EDITOR

void FRigVMTrait_ModuleEventDependency_MoverComponentTickFunctions::OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
	if (AnimNextComponent == nullptr)
	{
		return;
	}

	AActor* Actor = AnimNextComponent->GetOwner();
	if (Actor == nullptr)
	{
		return;
	}

	UMoverStandaloneLiaisonComponent* MoverStandaloneComponent = Actor->FindComponentByClass<UMoverStandaloneLiaisonComponent>();
	if (MoverStandaloneComponent == nullptr)
	{
		return;
	}

	if (FTickFunction* MoverDependentTickFunction = MoverStandaloneComponent->FindTickFunction(DependentMoverTickPhase))
	{
		// How should Mover tick be ordered, relative to the module function?
		if (Ordering == EAnimNextModuleEventDependencyOrdering::Before)
		{
			InContext.TickFunction.AddPrerequisite(MoverStandaloneComponent, *MoverDependentTickFunction);
		}
		else
		{
			MoverDependentTickFunction->AddPrerequisite(AnimNextComponent, InContext.TickFunction);
		}
	}
}


void FRigVMTrait_ModuleEventDependency_MoverComponentTickFunctions::OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
	if (AnimNextComponent == nullptr)
	{
		return;
	}

	AActor* Actor = AnimNextComponent->GetOwner();
	if (Actor == nullptr)
	{
		return;
	}

	UMoverStandaloneLiaisonComponent* MoverStandaloneComponent = Actor->FindComponentByClass<UMoverStandaloneLiaisonComponent>();
	if (MoverStandaloneComponent == nullptr)
	{
		return;
	}

	if (FTickFunction* MoverDependentTickFunction = MoverStandaloneComponent->FindTickFunction(DependentMoverTickPhase))
	{
		// How should Mover tick be ordered, relative to the module function?
		if (Ordering == EAnimNextModuleEventDependencyOrdering::Before)
		{
			InContext.TickFunction.RemovePrerequisite(MoverStandaloneComponent, *MoverDependentTickFunction);
		}
		else
		{
			MoverDependentTickFunction->RemovePrerequisite(AnimNextComponent, InContext.TickFunction);
		}
	}
}