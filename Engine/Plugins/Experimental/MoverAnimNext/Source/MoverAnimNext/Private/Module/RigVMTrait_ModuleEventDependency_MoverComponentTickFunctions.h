// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Module/RigVMTrait_ModuleEventDependency.h"
#include "MoverTypes.h"
#include "RigVMTrait_ModuleEventDependency_MoverComponentTickFunctions.generated.h"


/** A dependency on one of the tick functions that make up a MoverComponent's update flow. This uses 
 * the first-found MoverComponent on the current actor.
 * TODO: add support beyond standalone (non-networked) Mover backends
 */
USTRUCT(DisplayName = "Mover Component Tick Functions", meta = (ShowTooltip = true))
struct FRigVMTrait_ModuleEventDependency_MoverComponentTickFunctions : public FRigVMTrait_ModuleEventDependency
{
	GENERATED_BODY()

	// FRigVMTrait interface
#if WITH_EDITOR
	virtual FString GetDisplayName() const override;
#endif

	// FAnimNextModuleEventDependency interface
	virtual void OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;
	virtual void OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;


	// The Mover tick phase that this dependency relates to
	UPROPERTY(EditAnywhere, Category = "Dependency")
	EMoverTickPhase DependentMoverTickPhase = EMoverTickPhase::ApplyState;
};