// Copyright Epic Games, Inc. All Rights Reserved.
#include "MoverAnimNextModule.h"

#include "CoreMinimal.h"
#include "RigVMCore/RigVMRegistry.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverComponent.h"
#include "MoverPoseSearchTrajectoryPredictor.h"
#include "Animation/TrajectoryTypes.h"

#include "Modules/ModuleManager.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"


void FMoverAnimNextModule::StartupModule()
{	
	static UScriptStruct* const AllowedStructTypes[] =
	{
		FMoverPredictTrajectoryParams::StaticStruct(),	// Todo: remove
		FTrajectorySampleInfo::StaticStruct(),			// Todo: remove
		FPoseSearchTrajectoryData::StaticStruct(),
		FTransformTrajectory::StaticStruct(),
		TBaseStructure<FFrameRate>::Get()
	};

	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
		{ UMoverComponent::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
		{ UMoverTrajectoryPredictor::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },	// Doesn't seem like anim next supports interfaces
	};

	FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
	RigVMRegistry.RegisterStructTypes(AllowedStructTypes);
	RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);

}

void FMoverAnimNextModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

IMPLEMENT_MODULE(FMoverAnimNextModule, MoverAnimNext)